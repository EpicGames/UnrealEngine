// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SceneTypes.h"
#include "MaterialShaderQualitySettings.generated.h"

class UShaderPlatformQualitySettings;
enum EShaderPlatform : uint16;

//UCLASS(config = Engine, defaultconfig)
UCLASS(MinimalAPI)
class UMaterialShaderQualitySettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	MATERIALSHADERQUALITYSETTINGS_API UShaderPlatformQualitySettings* GetShaderPlatformQualitySettings(FName PlatformName);

	MATERIALSHADERQUALITYSETTINGS_API const UShaderPlatformQualitySettings* GetShaderPlatformQualitySettings(EShaderPlatform ShaderPlatform);

	MATERIALSHADERQUALITYSETTINGS_API bool HasPlatformQualitySettings(EShaderPlatform ShaderPlatform, EMaterialQualityLevel::Type QualityLevel);

#if WITH_EDITOR
	// Override GetShaderPlatformQualitySettings() return value with the specified platform's settings.
	// An empty PlatformName or otherwise non existent platform will cause GetShaderPlatformQualitySettings() 
	// to revert to its default behaviour.
	MATERIALSHADERQUALITYSETTINGS_API void SetPreviewPlatform(FName PlatformName);
	MATERIALSHADERQUALITYSETTINGS_API const FName& GetPreviewPlatform();
#endif

	static MATERIALSHADERQUALITYSETTINGS_API UMaterialShaderQualitySettings* Get();

private:
	UShaderPlatformQualitySettings* GetOrCreatePlatformSettings(FName ShaderPlatformName);

	UPROPERTY()
	TMap<FName, TObjectPtr<UShaderPlatformQualitySettings>> ForwardSettingMap;

#if WITH_EDITORONLY_DATA
	UShaderPlatformQualitySettings* PreviewPlatformSettings;
	FName PreviewPlatformName;
#endif

	static class UMaterialShaderQualitySettings* RenderQualitySingleton;
};
