// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if WITH_EDITOR
#include "Input/Reply.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "IDetailCustomization.h"
#endif

class IDetailLayoutBuilder;
struct FShaderQualityOverridesListItem;

#if WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// FMaterialShaderQualitySettingsCustomization

typedef SListView< TSharedPtr< struct FShaderQualityOverridesListItem > > SMaterialQualityOverridesListView;

DECLARE_DELEGATE(FOnUpdateMaterialShaderQuality);

class FMaterialShaderQualitySettingsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	MATERIALSHADERQUALITYSETTINGS_API static TSharedRef<IDetailCustomization> MakeInstance(const FOnUpdateMaterialShaderQuality InUpdateMaterials);

	FMaterialShaderQualitySettingsCustomization(const FOnUpdateMaterialShaderQuality InUpdateMaterials);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:

	TSharedRef<ITableRow> HandleGenerateQualityWidget(TSharedPtr< struct FShaderQualityOverridesListItem > InItem, const TSharedRef<STableViewBase>& OwnerTable);
	FReply UpdatePreviewShaders();
	FOnUpdateMaterialShaderQuality UpdateMaterials;
	FMaterialShaderQualitySettingsCustomization();
	TSharedPtr<SMaterialQualityOverridesListView> MaterialQualityOverridesListView;
	TArray< TSharedPtr< struct FShaderQualityOverridesListItem > >	QualityOverrideListSource;
};

#endif
