// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SceneTypes.h"
#include "ShaderPlatformQualitySettings.generated.h"

enum EShaderPlatform : uint16;

/**
* 
*/
UENUM()
enum class EMobileShadowQuality : uint8
{
	// Lowest quality, no filtering.
	NoFiltering,
	// Medium quality, 1x1 PCF filtering.
	PCF_1x1 UMETA(DisplayName = "1x1 PCF"),
	// Medium/High quality, 3x3 PCF filtering.
	PCF_3x3 UMETA(DisplayName = "3x3 PCF"),
	// Highest quality, 5x5 PCF filtering.
	PCF_5x5 UMETA(DisplayName = "5x5 PCF")
};

// FMaterialQualityOverrides represents the full set of possible material overrides per quality level.
USTRUCT()
struct FMaterialQualityOverrides
{
public:
	GENERATED_USTRUCT_BODY()
	
	FMaterialQualityOverrides() 
		: bDiscardQualityDuringCook(false)
		, bEnableOverride(false)
		, bForceFullyRough(false)
		, bForceNonMetal(false)
		, bForceDisableLMDirectionality(false)
		, bForceDisablePreintegratedGF(false)
		, bDisableMaterialNormalCalculation(false)
		, MobileShadowQuality(EMobileShadowQuality::PCF_3x3)
	{
	}

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Discard Quality During Cook"), Category = "Quality")
	bool bDiscardQualityDuringCook;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Enable Quality Override"), Category = "Quality")
	bool bEnableOverride;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Force Fully Rough"), Category = "Quality")
	bool bForceFullyRough;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Force Non-metal"), Category = "Quality")
	bool bForceNonMetal;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Disable Lightmap directionality"), Category = "Quality")
	bool bForceDisableLMDirectionality;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Force not use preintegrated GF for simple IBL"), Category = "Quality")
	bool bForceDisablePreintegratedGF;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Disable material normal calculation"), Category = "Quality")
	bool bDisableMaterialNormalCalculation;

	UPROPERTY(EditAnywhere, Config, Meta = (DisplayName = "Mobile shadow mapping quality"), Category = "Quality")
	EMobileShadowQuality MobileShadowQuality;

	MATERIALSHADERQUALITYSETTINGS_API bool CanOverride(EShaderPlatform ShaderPlatform) const;
	MATERIALSHADERQUALITYSETTINGS_API bool HasAnyOverridesSet() const;
};


UCLASS(config = Engine, defaultconfig, perObjectConfig, MinimalAPI)
class UShaderPlatformQualitySettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Config, Category="Quality")
	FMaterialQualityOverrides QualityOverrides[EMaterialQualityLevel::Num];

	FMaterialQualityOverrides& GetQualityOverrides(EMaterialQualityLevel::Type QualityLevel)
	{
		check(QualityLevel<EMaterialQualityLevel::Num);
		return QualityOverrides[(int32)QualityLevel];
	}

	MATERIALSHADERQUALITYSETTINGS_API const FMaterialQualityOverrides& GetQualityOverrides(EMaterialQualityLevel::Type QualityLevel) const;
	UE_DEPRECATED(5.8, "UShaderPlatformQualitySettings::BuildHash is unused in engine code and has been deprecated; use AppendToHashState instead if needed.")
	inline void BuildHash(EMaterialQualityLevel::Type QualityLevel, class FSHAHash& OutHash) const 
	{
		FSHA1 Builder;
		AppendToHashState(QualityLevel, Builder);
		OutHash = Builder.Finalize();
	}
	
	template <typename THasher>
	void AppendToHashState(EMaterialQualityLevel::Type QualityLevel, THasher& HashState) const
	{
		// since this hashes the full size of the raw struct, we must ensure there is no padding
		static_assert(std::has_unique_object_representations_v<FMaterialQualityOverrides>);

		const FMaterialQualityOverrides& QualityLevelOverrides = GetQualityOverrides(QualityLevel);
		HashState.Update((const uint8*)&QualityLevelOverrides, sizeof(QualityLevelOverrides));
	}
	
	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return ConfigPlatformName.IsEmpty() ? nullptr : *ConfigPlatformName;
	}

	FString ConfigPlatformName;
};
