// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MaterialShaderQualitySettings : ModuleRules
{
    public MaterialShaderQualitySettings(ReadOnlyTargetRules Target) : base(Target)
	{

        ShortName = "MSQS";

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
                "RHI",
			}
		);
        PrivateIncludePathModuleNames.Add("Engine");

        if (Target.bBuildEditor == true)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[] {            
       				"Slate",
				    "SlateCore",
                    "PropertyEditor",
                    "TargetPlatform",
                    "InputCore",
                }
            );

		}

	}
}
