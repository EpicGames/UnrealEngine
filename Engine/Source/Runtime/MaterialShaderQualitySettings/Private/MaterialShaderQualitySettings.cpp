// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialShaderQualitySettings.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Hash/xxhash.h"
#include "RHI.h"
#include "ShaderPlatformQualitySettings.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialShaderQualitySettings)

UMaterialShaderQualitySettings* UMaterialShaderQualitySettings::RenderQualitySingleton = nullptr;

UMaterialShaderQualitySettings::UMaterialShaderQualitySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMaterialShaderQualitySettings* UMaterialShaderQualitySettings::Get()
{
	if( RenderQualitySingleton == nullptr )
	{
		static const TCHAR* SettingsContainerName = TEXT("MaterialShaderQualitySettingsContainer");

		RenderQualitySingleton = FindObject<UMaterialShaderQualitySettings>(GetTransientPackage(), SettingsContainerName);

		if (RenderQualitySingleton == nullptr)
		{
			RenderQualitySingleton = NewObject<UMaterialShaderQualitySettings>(GetTransientPackage(), UMaterialShaderQualitySettings::StaticClass(), SettingsContainerName);
			RenderQualitySingleton->AddToRoot();
		}
	}
	return RenderQualitySingleton;
}

#if WITH_EDITOR
const FName& UMaterialShaderQualitySettings::GetPreviewPlatform()
{
	return PreviewPlatformName;
}

void UMaterialShaderQualitySettings::SetPreviewPlatform(FName PlatformName)
{
	 TObjectPtr<UShaderPlatformQualitySettings>* FoundPlatform = ForwardSettingMap.Find(PlatformName);
	 PreviewPlatformSettings = FoundPlatform == nullptr ? nullptr : FoundPlatform->Get();
	 PreviewPlatformName = PlatformName;
}
#endif

UShaderPlatformQualitySettings* UMaterialShaderQualitySettings::GetOrCreatePlatformSettings(FName ShaderPlatformName)
{
	auto* PlatformSettings = ForwardSettingMap.Find(ShaderPlatformName);
	if (PlatformSettings == nullptr)
	{
		FString ObjectName("ForwardShadingQuality_");
		ShaderPlatformName.AppendString(ObjectName);

		auto* ForwardQualitySettings = FindObject<UShaderPlatformQualitySettings>(this, *ObjectName);
		if (ForwardQualitySettings == nullptr)
		{
		
			FName ForwardSettingsName(*ObjectName);
			ForwardQualitySettings = NewObject<UShaderPlatformQualitySettings>(this, UShaderPlatformQualitySettings::StaticClass(), FName(*ObjectName));
// if we aren't running in the editor, then we will only want to use our own settings
#if WITH_EDITOR
			ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), ShaderPlatformName);
			ForwardQualitySettings->ConfigPlatformName = TargetPlatform ? TargetPlatform->IniPlatformName() : FString();
#endif
			ForwardQualitySettings->LoadConfig();
		}

		return ForwardSettingMap.Add(ShaderPlatformName, ForwardQualitySettings);
	}
	return *PlatformSettings;
}

static const FName GetPlatformNameFromShaderPlatform(EShaderPlatform Platform)
{
	return LegacyShaderPlatformToShaderFormat(Platform);
}

bool UMaterialShaderQualitySettings::HasPlatformQualitySettings(EShaderPlatform ShaderPlatform, EMaterialQualityLevel::Type QualityLevel)
{
	const UShaderPlatformQualitySettings* PlatformShaderPlatformQualitySettings = GetShaderPlatformQualitySettings(ShaderPlatform);
	const FMaterialQualityOverrides& PlatFormQualityOverrides = PlatformShaderPlatformQualitySettings->GetQualityOverrides(QualityLevel);
	return PlatFormQualityOverrides.bEnableOverride && PlatFormQualityOverrides.HasAnyOverridesSet();
}

const UShaderPlatformQualitySettings* UMaterialShaderQualitySettings::GetShaderPlatformQualitySettings(EShaderPlatform ShaderPlatform)
{
 #if WITH_EDITORONLY_DATA
	// TODO: discuss this, in order to preview render quality settings we override the
	// requested platform's settings.
	// However we do not know if we are asking for the editor preview window (override able) or for thumbnails, cooking purposes etc.. (Must not override)
	// The code below 'works' because desktop platforms do not cook for ES31 preview.
	if (IsPCPlatform(ShaderPlatform) && GetMaxSupportedFeatureLevel(ShaderPlatform) <= ERHIFeatureLevel::ES3_1)
	{
		// Can check this cant be cooked by iterating through target platforms and shader formats to ensure it's not covered.
		if (PreviewPlatformSettings != nullptr)
		{
			return PreviewPlatformSettings;
		}
	}
#endif
	return GetShaderPlatformQualitySettings(GetPlatformNameFromShaderPlatform(ShaderPlatform));
}

UShaderPlatformQualitySettings* UMaterialShaderQualitySettings::GetShaderPlatformQualitySettings(FName PlatformName)
{
	return GetOrCreatePlatformSettings(PlatformName);
}

//////////////////////////////////////////////////////////////////////////

UShaderPlatformQualitySettings::UShaderPlatformQualitySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// high quality overrides are always enabled by default
	check(IsInGameThread());
	GetQualityOverrides(EMaterialQualityLevel::High).bEnableOverride = true;
}

const FMaterialQualityOverrides& UShaderPlatformQualitySettings::GetQualityOverrides(EMaterialQualityLevel::Type QualityLevel) const
{
	check(QualityLevel <= EMaterialQualityLevel::Num);
	static const FMaterialQualityOverrides DefaultQualityOverrides;
	return (QualityLevel < EMaterialQualityLevel::Num) ? QualityOverrides[(int32)QualityLevel] : DefaultQualityOverrides;
}

//////////////////////////////////////////////////////////////////////////

bool FMaterialQualityOverrides::CanOverride(EShaderPlatform ShaderPlatform) const
{
	// Only mobile renderer can lower the quality of a shader even without quality level nodes in the material (see TMobileBasePassPSPolicyParamType<>::ModifyCompilationEnvironmentForQualityLevel).
	// Opt-in the platforms here that are going to use it.
	return IsMobilePlatform(ShaderPlatform);
}

bool FMaterialQualityOverrides::HasAnyOverridesSet() const
{
	static const FMaterialQualityOverrides DefaultOverrides;

	return
		MobileShadowQuality != DefaultOverrides.MobileShadowQuality
		|| bForceDisableLMDirectionality != DefaultOverrides.bForceDisableLMDirectionality
		|| bForceFullyRough != DefaultOverrides.bForceFullyRough
		|| bForceNonMetal != DefaultOverrides.bForceNonMetal
		|| bForceDisablePreintegratedGF != DefaultOverrides.bForceDisablePreintegratedGF
		|| bDisableMaterialNormalCalculation != DefaultOverrides.bDisableMaterialNormalCalculation
		|| bDiscardQualityDuringCook != DefaultOverrides.bDiscardQualityDuringCook;
}

