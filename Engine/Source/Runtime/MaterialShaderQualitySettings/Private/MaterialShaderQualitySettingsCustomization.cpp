// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialShaderQualitySettingsCustomization.h"

#if WITH_EDITOR

#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "ShaderQualityOverridesListItem.h"
#include "ShaderPlatformQualitySettings.h"

#define LOCTEXT_NAMESPACE "MaterialShaderQualitySettings"

//////////////////////////////////////////////////////////////////////////
// FMaterialShaderQualitySettingsCustomization
namespace FMaterialShaderQualitySettingsCustomizationConstants
{
	const FText DisabledTip = LOCTEXT("GitHubSourceRequiredToolTip", "This requires GitHub source.");
}

TSharedRef<IDetailCustomization> FMaterialShaderQualitySettingsCustomization::MakeInstance(const FOnUpdateMaterialShaderQuality InUpdateMaterials)
{
	return MakeShareable(new FMaterialShaderQualitySettingsCustomization(InUpdateMaterials));
}

FMaterialShaderQualitySettingsCustomization::FMaterialShaderQualitySettingsCustomization(const FOnUpdateMaterialShaderQuality InUpdateMaterials)
{
	UpdateMaterials = InUpdateMaterials;
}

class SQualityListItem
	: public SMultiColumnTableRow< TSharedPtr<struct FShaderQualityOverridesListItem> >
{
public:

	SLATE_BEGIN_ARGS(SQualityListItem) { }
	SLATE_ARGUMENT(TSharedPtr<FShaderQualityOverridesListItem>, Item)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		check(Item.IsValid());
		const auto args = FSuperRowType::FArguments();
		SMultiColumnTableRow< TSharedPtr<FShaderQualityOverridesListItem> >::Construct(args, InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TEXT("Quality Option"))
		{
			return	SNew(SBox)
				.HeightOverride(20)
				.Padding(FMargin(3, 0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Item->RangeName))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}
		else
		{
			EMaterialQualityLevel::Type MaterialQualityLevel = EMaterialQualityLevel::Num;

			if (ColumnName == TEXT("Low"))
			{
				MaterialQualityLevel = EMaterialQualityLevel::Low;
			}
			else if (ColumnName == TEXT("Medium"))
			{
				MaterialQualityLevel = EMaterialQualityLevel::Medium;
			}
			else if (ColumnName == TEXT("High"))
			{
				MaterialQualityLevel = EMaterialQualityLevel::High;
			}
			else if (ColumnName == TEXT("Epic"))
			{
				MaterialQualityLevel = EMaterialQualityLevel::Epic;
			}

			if (MaterialQualityLevel < EMaterialQualityLevel::Num)
			{
				TSharedRef<SWidget> Widget = Item->OverrideHandles.FindChecked(MaterialQualityLevel)->CreatePropertyValueWidget();
				Widget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SQualityListItem::IsEnabled, MaterialQualityLevel)));
				return Widget;
			}
		}
	
		return SNullWidget::NullWidget;
	}

private:

	bool IsEnabled(EMaterialQualityLevel::Type QualityLevel) const
	{
		const TSharedRef<IPropertyHandle> ItemHandle = Item->OverrideHandles.FindChecked(QualityLevel);

		// We always allow quality level forced disables
		static FName FNAME_DiscardQualityDuringCook = GET_MEMBER_NAME_CHECKED(FMaterialQualityOverrides,bDiscardQualityDuringCook);
		if (ItemHandle->GetProperty()->GetFName() == FNAME_DiscardQualityDuringCook)
		{
			return true;
		}

		// bEnableOverride is always enabled for all levels other than High and disabled for High.
		static FName FNAME_EnableOverride = GET_MEMBER_NAME_CHECKED(FMaterialQualityOverrides,bEnableOverride);
		if (ItemHandle->GetProperty()->GetFName() == FNAME_EnableOverride)
		{
			return QualityLevel != EMaterialQualityLevel::High;
		}

		// enable only if the enabled checkbox is checked
		const TSharedRef<IPropertyHandle> EnabledHandle = Item->EnabledHandles.FindChecked(QualityLevel);
		bool bEnabled = false;
		EnabledHandle->GetValue(bEnabled);
		return bEnabled;
	}

	TSharedPtr< FShaderQualityOverridesListItem > Item;
};

TSharedRef<ITableRow> FMaterialShaderQualitySettingsCustomization::HandleGenerateQualityWidget(TSharedPtr<FShaderQualityOverridesListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SQualityListItem, OwnerTable)
		.Item(InItem);
}

FReply FMaterialShaderQualitySettingsCustomization::UpdatePreviewShaders()
{
	UpdateMaterials.ExecuteIfBound();
	return FReply::Handled();
}

void FMaterialShaderQualitySettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& ForwardRenderingCategory = DetailLayout.EditCategory(TEXT("Forward Rendering Overrides"));

	// Build a list of FShaderQualityRange properties.
	QualityOverrideListSource.Empty();
	const TSharedRef<IPropertyHandle> QualityOverridesArray = DetailLayout.GetProperty(FName("QualityOverrides"));
	DetailLayout.HideProperty(QualityOverridesArray);
	uint32 NumChildren;
	QualityOverridesArray->GetNumChildren(NumChildren);
	check(NumChildren == (uint32)EMaterialQualityLevel::Num);

	// Find the QualityOverrides array property
	const TSharedRef< IPropertyHandle > QualityOverrides0 = QualityOverridesArray->GetChildHandle(0).ToSharedRef();
	QualityOverrides0->GetNumChildren(NumChildren);

	// We will store the handles for the bEnableOverride properties for each QL.
	TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>> EnabledHandles;

	for (uint32 OverrideIndex = 0; OverrideIndex < NumChildren; OverrideIndex++)
	{
		FString DisplayName;
		TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>> OverrideHandles;
		for (uint32 QualityIndex = 0; QualityIndex < EMaterialQualityLevel::Num; QualityIndex++)
		{
			TSharedRef< IPropertyHandle > OverrideHandle = QualityOverridesArray->GetChildHandle(QualityIndex)->GetChildHandle(OverrideIndex).ToSharedRef();
			OverrideHandles.Add((EMaterialQualityLevel::Type)QualityIndex, OverrideHandle);
			if (QualityIndex == 0)
			{
				DisplayName = OverrideHandle->GetProperty()->GetMetaData("DisplayName");
			}
		}

		// Remember the bEnableOverride properties so they can be referenced when deciding whether to enable the widgets
		if (OverrideIndex == 1)
		{
			EnabledHandles = OverrideHandles;
		}

		QualityOverrideListSource.Add(MakeShareable(new FShaderQualityOverridesListItem(DisplayName, OverrideHandles, EnabledHandles)));
	}

	ForwardRenderingCategory.AddCustomRow(LOCTEXT("ForwardRenderingMaterialOverrides", "Forward Rendering Material Overrides"))
		[
			SAssignNew(MaterialQualityOverridesListView, SMaterialQualityOverridesListView)
			.ListItemsSource(&QualityOverrideListSource)
			.OnGenerateRow(this, &FMaterialShaderQualitySettingsCustomization::HandleGenerateQualityWidget)
			.SelectionMode(ESelectionMode::None)
			.HeaderRow(
			SNew(SHeaderRow)
			// 
			+ SHeaderRow::Column("Quality Option")
			.HAlignCell(HAlign_Left)
			.FillWidth(1)
			.HeaderContentPadding(FMargin(0, 3))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialQualityList_Category", "Quality Option"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			// 
			+ SHeaderRow::Column("Low")
				.HAlignCell(HAlign_Left)
				.FillWidth(1)
				.HeaderContentPadding(FMargin(0, 3))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialQualityList_Low", "Low"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			// 
			+ SHeaderRow::Column("Medium")
				.HAlignCell(HAlign_Left)
				.FillWidth(1)
				.HeaderContentPadding(FMargin(0, 3))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialQualityList_Medium", "Medium"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			// 
			+ SHeaderRow::Column("High")
				.HAlignCell(HAlign_Left)
				.FillWidth(1)
				.HeaderContentPadding(FMargin(0, 3))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialQualityList_High", "High"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			//
			+ SHeaderRow::Column("Epic")
				.HAlignCell(HAlign_Left)
				.FillWidth(1)
				.HeaderContentPadding(FMargin(0, 3))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialQualityList_Epic", "Epic"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			)
		];

	ForwardRenderingCategory.AddCustomRow(LOCTEXT("ForwardRenderingMaterialOverrides", "Forward Rendering Material Overrides"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.Text(LOCTEXT("UpdatePreviewShaders", "Update preview shaders"))
			.ToolTipText(LOCTEXT("UpdatePreviewShadersButton_Tooltip", "Updates the editor to reflect changes to quality settings."))
			.OnClicked(this, &FMaterialShaderQualitySettingsCustomization::UpdatePreviewShaders)
		]
	];

}


//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
