// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "SceneTypes.h"

// FShaderQualityOverridesListItem
// Helper struct for FMaterialShaderQualitySettingsCustomization, contains info required to populate a material quality row.
struct FShaderQualityOverridesListItem
{
public:
	FString RangeName;
	
	// Property handles for this item's each override setting for each QL
	TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>> OverrideHandles;
	
	// Property handles for each QL's bEnabled flag, used to determine if this item's widgets should be enabled.
	TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>> EnabledHandles;

	FShaderQualityOverridesListItem() {}

	FShaderQualityOverridesListItem(FString InRangeName, const TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>>& InOverrideHandles, const TMap<EMaterialQualityLevel::Type, TSharedRef<IPropertyHandle>> &InEnabledHandles)
	: RangeName(InRangeName)
	, OverrideHandles(InOverrideHandles)
	, EnabledHandles(InEnabledHandles)
	{
	}
};
#endif